// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AssetEditorMessages.generated.h"


/**
 * Implements a message for opening an asset in the asset browser.
 */
USTRUCT()
struct FAssetEditorRequestOpenAsset
{
	GENERATED_USTRUCT_BODY()
	
	/**
	 * Holds the name of the asset to open.
	 */
	UPROPERTY(EditAnywhere, Category="Message")
	FString AssetName;


	/** Default constructor. */
	FAssetEditorRequestOpenAsset() { }

	/** Creates and initializes a new instance. */
	FAssetEditorRequestOpenAsset(const FString& InAssetName)
		: AssetName(InAssetName)
	{ }
};
