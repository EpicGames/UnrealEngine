// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "FileServerMessages.generated.h"


/**
 * Implements a message that is published by file servers when they're ready to accept connections.
 */
USTRUCT()
struct FFileServerReady
{
	GENERATED_USTRUCT_BODY()
	
	/** Holds the list of IP addresses that the file server is listening on. */
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<FString> AddressList;
	
	/** Holds the file server's application identifier. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid InstanceId;

	/** Default constructor. */
	FFileServerReady() { }

	/** Creates and initializes a new instance. */
	FFileServerReady(const TArray<FString>& InAddressList, const FGuid& InInstanceId)
		: AddressList(InAddressList)
		, InstanceId(InInstanceId)
	{ }
};
