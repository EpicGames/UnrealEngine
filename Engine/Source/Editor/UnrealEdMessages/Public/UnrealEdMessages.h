// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Dependencies
 *****************************************************************************/

#include "CoreUObject.h"


/* Messages
 *****************************************************************************/

#include "AssetEditorMessages.h"
#include "FileServerMessages.h"
